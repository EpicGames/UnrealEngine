// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.All)]
public class ChaosUserDataPTTestsTarget : TestTargetRules
{
	public ChaosUserDataPTTestsTarget(TargetInfo Target) : base(Target)
	{
	}
}
