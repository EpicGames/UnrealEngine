// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "TestHarness.h"
#include "ChaosUserDataPT.h"
#include "PBDRigidsSolver.h"
#include "ChaosSolversModule.h"
#include "Chaos/Box.h"

class FTestUserData : public Chaos::TUserDataManagerPT<FString> { };

class FTestUserData_GetDataGT : public Chaos::TUserDataManagerPT<FString>
{
public:
	FTestUserData_GetDataGT() : TUserDataManagerPT(Chaos::FUserDataPTConfig{ .bGetData_GT = true }) {}
};


// Advance a solver once with the provided DeltaTime, then wait for any async tasks to finish before continuing
void AdvanceAndWait(Chaos::FPBDRigidsSolver* InSolver, float DeltaTime)
{
	if(InSolver)
	{
		InSolver->AdvanceAndDispatch_External(DeltaTime);
		InSolver->WaitOnPendingTasks_External();
	}
}

TEST_CASE("ChaosUserDataPT", "[integration]")
{
	const float DeltaTime = 1.f;
	const FString TestString1 = TEXT("TestData1");
	const FString TestString2 = TEXT("TestData2");
	const FString TestString3 = TEXT("TestData3");

	// Create a solver in the solvers module
	FChaosSolversModule* Module = FChaosSolversModule::GetModule();
	Chaos::FPBDRigidsSolver* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/DeltaTime, Chaos::EThreadingMode::TaskGraph);

	// Create a test userdata manager in the solver
	FTestUserData* TestUserData = Solver->CreateAndRegisterSimCallbackObject_External<FTestUserData>();

	// Make a box geometry
	auto BoxGeom = Chaos::FImplicitObjectPtr(new Chaos::TBox<Chaos::FReal, 3>(Chaos::FVec3(-1, -1, -1), Chaos::FVec3(1, 1, 1)));

	// Add some proxies to the solver
	Chaos::FSingleParticlePhysicsProxy* Proxy0 = Chaos::FSingleParticlePhysicsProxy::Create(Chaos::FGeometryParticle::CreateParticle());
	Chaos::FSingleParticlePhysicsProxy* Proxy1 = Chaos::FSingleParticlePhysicsProxy::Create(Chaos::FGeometryParticle::CreateParticle());
	Chaos::FSingleParticlePhysicsProxy* Proxy2 = Chaos::FSingleParticlePhysicsProxy::Create(Chaos::FGeometryParticle::CreateParticle());
	Chaos::FRigidBodyHandle_External& HandleExternal0 = Proxy0->GetGameThreadAPI();
	Chaos::FRigidBodyHandle_External& HandleExternal1 = Proxy1->GetGameThreadAPI();
	Chaos::FRigidBodyHandle_External& HandleExternal2 = Proxy2->GetGameThreadAPI();
	HandleExternal0.SetGeometry(BoxGeom);
	HandleExternal1.SetGeometry(BoxGeom);
	HandleExternal2.SetGeometry(BoxGeom);
	Solver->RegisterObject(Proxy0);
	Solver->RegisterObject(Proxy1);
	Solver->RegisterObject(Proxy2);

	// Advance the solver twice to make sure the PT handles are created and in the evolution
	AdvanceAndWait(Solver, DeltaTime);
	AdvanceAndWait(Solver, DeltaTime);

	// Add data
	SECTION("Data propagates from GT to PT")
	{
		// Add userdata to the particle
		TestUserData->SetData_GT(HandleExternal0, TestString1);

		// The first callback should show no data because the ensure will occur before
		// the sim callback has occurred.
		Solver->EnqueueCommandImmediate([&]()
		{
			REQUIRE(TestUserData->GetData_PT(*Proxy0->GetPhysicsThreadAPI()) == nullptr);
		});
		AdvanceAndWait(Solver, DeltaTime);

		// The data should make it to the physics thread by this point
		Solver->EnqueueCommandImmediate([&]()
		{
			REQUIRE(*TestUserData->GetData_PT(*Proxy0->GetPhysicsThreadAPI()) == TestString1);
		});
		AdvanceAndWait(Solver, DeltaTime);
	}

	// Change data
	SECTION("Data updates propagate from GT to PT")
	{
		// Add userdata to the particle
		TestUserData->SetData_GT(HandleExternal0, TestString1);
		AdvanceAndWait(Solver, DeltaTime);

		// Set the userdata to something else
		TestUserData->SetData_GT(HandleExternal0, TestString2);
		AdvanceAndWait(Solver, DeltaTime);

		// The data should make it to the physics thread by this point
		Solver->EnqueueCommandImmediate([&]()
		{
			REQUIRE(*TestUserData->GetData_PT(*Proxy0->GetPhysicsThreadAPI()) == TestString2);
		});
		AdvanceAndWait(Solver, DeltaTime);
	}

	// Delete data
	SECTION("Data removals propagate from GT to PT")
	{
		// Add userdata to the particle and advance it to the physics thread
		TestUserData->SetData_GT(HandleExternal0, TestString1);
		AdvanceAndWait(Solver, DeltaTime);

		// Delete the data
		TestUserData->RemoveData_GT(HandleExternal0);

		// Data should exist for one more update
		Solver->EnqueueCommandImmediate([&]()
		{
			REQUIRE(*TestUserData->GetData_PT(*Proxy0->GetPhysicsThreadAPI()) == TestString1);
		});
		AdvanceAndWait(Solver, DeltaTime);

		// Data should be deleted at this point
		Solver->EnqueueCommandImmediate([&]()
		{
			REQUIRE(TestUserData->GetData_PT(*Proxy0->GetPhysicsThreadAPI()) == nullptr);
		});
		AdvanceAndWait(Solver, DeltaTime);
	}

	// Delete data from particle that doesn't have it
	SECTION("Removing data from a particle that never had data set on it does nothing")
	{
		// Delete data that isn't there
		TestUserData->RemoveData_GT(HandleExternal0);
		AdvanceAndWait(Solver, DeltaTime);
		Solver->EnqueueCommandImmediate([&]()
		{
			REQUIRE(TestUserData->GetData_PT(*Proxy0->GetPhysicsThreadAPI()) == nullptr);
		});
		AdvanceAndWait(Solver, DeltaTime);
	}

	// Make sure a particle with a recycled index can't get a deleted particle's userdata
	SECTION("Deleting a particle that has userdata associated with it should remove the userdata")
	{
		// Add data to a particle, make sure it gets to PT, then delete the particle.
		const Chaos::FUniqueIdx UniqueIdx0 = HandleExternal0.UniqueIdx();
		TestUserData->SetData_GT(HandleExternal0, TestString1);
		AdvanceAndWait(Solver, DeltaTime);
		Solver->UnregisterObject(Proxy0);
		AdvanceAndWait(Solver, DeltaTime);
		AdvanceAndWait(Solver, DeltaTime);

		struct FMockHandle
		{
			FMockHandle(Chaos::FUniqueIdx InUniqueIdx) : MUniqueIdx(InUniqueIdx) { }
			Chaos::FUniqueIdx UniqueIdx() const { return MUniqueIdx; }
			Chaos::FUniqueIdx MUniqueIdx;
		};

		// Access userdata with the invalid particle handle - it should retrieve nothing
		Solver->EnqueueCommandImmediate([&]()
		{
			const FMockHandle MockHandle0 = FMockHandle(UniqueIdx0);
			REQUIRE(TestUserData->GetData_PT(MockHandle0) == nullptr);
		});
		AdvanceAndWait(Solver, DeltaTime);
	}

	SECTION("Clearing all data from a userdata manager")
	{
		// Add data to three particles, propagate it to the PT
		TestUserData->SetData_GT(HandleExternal0, TestString1);
		TestUserData->SetData_GT(HandleExternal1, TestString1);
		TestUserData->SetData_GT(HandleExternal2, TestString1);
		AdvanceAndWait(Solver, DeltaTime);
		AdvanceAndWait(Solver, DeltaTime);

		// Clear all data from the userdata manager, but after that set data back on
		// particle 2 - the fact that it happened _after_ clearing should mean it is
		// still there for particle 2 after the clear reaches the PT.
		TestUserData->ClearData_GT();
		TestUserData->SetData_GT(HandleExternal2, TestString1);

		// Check to see that the data is still there at first
		Solver->EnqueueCommandImmediate([&]()
		{
			REQUIRE(*TestUserData->GetData_PT(*Proxy0->GetPhysicsThreadAPI()) == TestString1);
			REQUIRE(*TestUserData->GetData_PT(*Proxy1->GetPhysicsThreadAPI()) == TestString1);
			REQUIRE(*TestUserData->GetData_PT(*Proxy2->GetPhysicsThreadAPI()) == TestString1);
		});
		AdvanceAndWait(Solver, DeltaTime);

		// Check make sure that after a couple updates, the data is cleared
		Solver->EnqueueCommandImmediate([&]()
		{
			REQUIRE(TestUserData->GetData_PT(*Proxy0->GetPhysicsThreadAPI()) == nullptr);
			REQUIRE(TestUserData->GetData_PT(*Proxy1->GetPhysicsThreadAPI()) == nullptr);
			REQUIRE(*TestUserData->GetData_PT(*Proxy2->GetPhysicsThreadAPI()) == TestString1);
		});
		AdvanceAndWait(Solver, DeltaTime);
	}
	
	SECTION("Multiple particles with different data")
	{
		TestUserData->SetData_GT(HandleExternal0, TestString1);
		TestUserData->SetData_GT(HandleExternal1, TestString2);
		TestUserData->SetData_GT(HandleExternal2, TestString3);
		AdvanceAndWait(Solver, DeltaTime);
		AdvanceAndWait(Solver, DeltaTime);

		Solver->EnqueueCommandImmediate([&]()
		{
			REQUIRE(*TestUserData->GetData_PT(*Proxy0->GetPhysicsThreadAPI()) == TestString1);
			REQUIRE(*TestUserData->GetData_PT(*Proxy1->GetPhysicsThreadAPI()) == TestString2);
			REQUIRE(*TestUserData->GetData_PT(*Proxy2->GetPhysicsThreadAPI()) == TestString3);
		});
		AdvanceAndWait(Solver, DeltaTime);
	}
	
	SECTION("Set and remove in same frame results in no data on PT")
	{
		TestUserData->SetData_GT(HandleExternal0, TestString1);
		TestUserData->RemoveData_GT(HandleExternal0);
		AdvanceAndWait(Solver, DeltaTime);
		AdvanceAndWait(Solver, DeltaTime);

		Solver->EnqueueCommandImmediate([&]()
		{
			REQUIRE(TestUserData->GetData_PT(*Proxy0->GetPhysicsThreadAPI()) == nullptr);
		});
		AdvanceAndWait(Solver, DeltaTime);
	}

	SECTION("Remove and re-add in same frame preserves new value")
	{
		TestUserData->SetData_GT(HandleExternal0, TestString1);
		AdvanceAndWait(Solver, DeltaTime);
		AdvanceAndWait(Solver, DeltaTime);

		TestUserData->RemoveData_GT(HandleExternal0);
		TestUserData->SetData_GT(HandleExternal0, TestString2);
		AdvanceAndWait(Solver, DeltaTime);

		Solver->EnqueueCommandImmediate([&]()
		{
			REQUIRE(*TestUserData->GetData_PT(*Proxy0->GetPhysicsThreadAPI()) == TestString2);
		});
		AdvanceAndWait(Solver, DeltaTime);
	}

	SECTION("VisitData_PT iterates over all stored entries")
	{
		TestUserData->SetData_GT(HandleExternal0, TestString1);
		TestUserData->SetData_GT(HandleExternal1, TestString2);
		TestUserData->SetData_GT(HandleExternal2, TestString3);
		AdvanceAndWait(Solver, DeltaTime);
		AdvanceAndWait(Solver, DeltaTime);

		Solver->EnqueueCommandImmediate([&]()
		{
			TMap<int32, FString> Visited;
			TestUserData->VisitData_PT([&Visited](Chaos::FUniqueIdx Idx, const FString& Data)
			{
				Visited.Add(Idx.Idx, Data);
			});

			REQUIRE(Visited.Num() == 3);
			REQUIRE(Visited[Proxy0->GetPhysicsThreadAPI()->UniqueIdx().Idx] == TestString1);
			REQUIRE(Visited[Proxy1->GetPhysicsThreadAPI()->UniqueIdx().Idx] == TestString2);
			REQUIRE(Visited[Proxy2->GetPhysicsThreadAPI()->UniqueIdx().Idx] == TestString3);
		});
		AdvanceAndWait(Solver, DeltaTime);
	}

	SECTION("GetData_GT returns nullptr when bGetData_GT is false")
	{
		TestUserData->SetData_GT(HandleExternal0, TestString1);
		REQUIRE(TestUserData->GetData_GT(HandleExternal0) == nullptr);
	}

	Solver->WaitOnPendingTasks_External();
	Module->DestroySolver(Solver);
}


TEST_CASE("FTestUserData_GetDataGT", "[integration]")
{
	const float DeltaTime = 1.f;
	const FString TestString1 = TEXT("TestData1");
	const FString TestString2 = TEXT("TestData2");

	FChaosSolversModule* Module = FChaosSolversModule::GetModule();
	Chaos::FPBDRigidsSolver* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/DeltaTime, Chaos::EThreadingMode::TaskGraph);
	FTestUserData_GetDataGT* TestUserData = Solver->CreateAndRegisterSimCallbackObject_External<FTestUserData_GetDataGT>();

	auto BoxGeom = Chaos::FImplicitObjectPtr(new Chaos::TBox<Chaos::FReal, 3>(Chaos::FVec3(-1, -1, -1), Chaos::FVec3(1, 1, 1)));

	Chaos::FSingleParticlePhysicsProxy* Proxy0 = Chaos::FSingleParticlePhysicsProxy::Create(Chaos::FGeometryParticle::CreateParticle());
	Chaos::FSingleParticlePhysicsProxy* Proxy1 = Chaos::FSingleParticlePhysicsProxy::Create(Chaos::FGeometryParticle::CreateParticle());
	Chaos::FRigidBodyHandle_External& HandleExternal0 = Proxy0->GetGameThreadAPI();
	Chaos::FRigidBodyHandle_External& HandleExternal1 = Proxy1->GetGameThreadAPI();
	HandleExternal0.SetGeometry(BoxGeom);
	HandleExternal1.SetGeometry(BoxGeom);
	Solver->RegisterObject(Proxy0);
	Solver->RegisterObject(Proxy1);

	AdvanceAndWait(Solver, DeltaTime);
	AdvanceAndWait(Solver, DeltaTime);

	SECTION("Data updates are accessible from GT with bGetData_GT = true")
	{
		const FString* StringPtr = TestUserData->GetData_GT(HandleExternal0);
		REQUIRE(StringPtr == nullptr);

		TestUserData->SetData_GT(HandleExternal0, TestString1);
		StringPtr = TestUserData->GetData_GT(HandleExternal0);
		REQUIRE(*StringPtr == TestString1);

		AdvanceAndWait(Solver, DeltaTime);

		TestUserData->SetData_GT(HandleExternal0, TestString2);
		StringPtr = TestUserData->GetData_GT(HandleExternal0);
		REQUIRE(*StringPtr == TestString2);
	}

	SECTION("GT data is removed properly")
	{
		TestUserData->SetData_GT(HandleExternal0, TestString1);
		TestUserData->SetData_GT(HandleExternal1, TestString2);
		const FString* StringPtr0 = TestUserData->GetData_GT(HandleExternal0);
		const FString* StringPtr1 = TestUserData->GetData_GT(HandleExternal1);
		REQUIRE(*StringPtr0 == TestString1);
		REQUIRE(*StringPtr1 == TestString2);

		TestUserData->RemoveData_GT(HandleExternal0);
		StringPtr0 = TestUserData->GetData_GT(HandleExternal0);
		StringPtr1 = TestUserData->GetData_GT(HandleExternal1);
		REQUIRE(StringPtr0 == nullptr);
		REQUIRE(*StringPtr1 == TestString2);

		TestUserData->ClearData_GT();
		StringPtr0 = TestUserData->GetData_GT(HandleExternal0);
		StringPtr1 = TestUserData->GetData_GT(HandleExternal1);
		REQUIRE(StringPtr0 == nullptr);
		REQUIRE(StringPtr1 == nullptr);
	}

	SECTION("GT data also propagates to PT")
	{
		TestUserData->SetData_GT(HandleExternal0, TestString1);
		AdvanceAndWait(Solver, DeltaTime);

		REQUIRE(*TestUserData->GetData_GT(HandleExternal0) == TestString1);

		Solver->EnqueueCommandImmediate([&]()
		{
			REQUIRE(*TestUserData->GetData_PT(*Proxy0->GetPhysicsThreadAPI()) == TestString1);
		});
		AdvanceAndWait(Solver, DeltaTime);
	}

	SECTION("GT data is immediately removed when particle is unregistered")
	{
		// Set data on both particles
		TestUserData->SetData_GT(HandleExternal0, TestString1);
		TestUserData->SetData_GT(HandleExternal1, TestString2);
		REQUIRE(*TestUserData->GetData_GT(HandleExternal0) == TestString1);
		REQUIRE(*TestUserData->GetData_GT(HandleExternal1) == TestString2);

		// Save the unique index before unregistering, since the handle
		// will be invalid after UnregisterObject
		struct FMockHandle
		{
			FMockHandle(Chaos::FUniqueIdx InUniqueIdx) : MUniqueIdx(InUniqueIdx) { }
			Chaos::FUniqueIdx UniqueIdx() const { return MUniqueIdx; }
			Chaos::FUniqueIdx MUniqueIdx;
		};
		const FMockHandle MockHandle0(HandleExternal0.UniqueIdx());

		// Unregister particle 0. The GT delegate should fire synchronously
		// inside UnregisterObject, cleaning the GT cache immediately.
		Solver->UnregisterObject(Proxy0);

		// GT cache for the unregistered particle should already be gone —
		// no solver advance needed.
		REQUIRE(TestUserData->GetData_GT(MockHandle0) == nullptr);

		// The other particle's GT data should be unaffected
		REQUIRE(*TestUserData->GetData_GT(HandleExternal1) == TestString2);

		// Let the solver process the destruction so PT is also cleaned up
		AdvanceAndWait(Solver, DeltaTime);
		AdvanceAndWait(Solver, DeltaTime);
	}

	SECTION("Recycled unique index does not return stale GT data")
	{
		// Set data on particle 0
		TestUserData->SetData_GT(HandleExternal0, TestString1);
		REQUIRE(*TestUserData->GetData_GT(HandleExternal0) == TestString1);

		// Save the unique index, then destroy the particle
		const Chaos::FUniqueIdx OldUniqueIdx = HandleExternal0.UniqueIdx();
		Solver->UnregisterObject(Proxy0);
		AdvanceAndWait(Solver, DeltaTime);
		AdvanceAndWait(Solver, DeltaTime);

		// Create a new particle — it may or may not reuse the same
		// FUniqueIdx, but regardless, querying the old index should not
		// return stale data.
		struct FMockHandle
		{
			FMockHandle(Chaos::FUniqueIdx InUniqueIdx) : MUniqueIdx(InUniqueIdx) { }
			Chaos::FUniqueIdx UniqueIdx() const { return MUniqueIdx; }
			Chaos::FUniqueIdx MUniqueIdx;
		};
		const FMockHandle OldHandle(OldUniqueIdx);
		REQUIRE(TestUserData->GetData_GT(OldHandle) == nullptr);

		// If we create a brand-new particle and set fresh data on it,
		// GetData_GT must return the new data, not anything from the old particle.
		Chaos::FSingleParticlePhysicsProxy* Proxy2 = Chaos::FSingleParticlePhysicsProxy::Create(Chaos::FGeometryParticle::CreateParticle());
		Chaos::FRigidBodyHandle_External& HandleExternal2 = Proxy2->GetGameThreadAPI();
		HandleExternal2.SetGeometry(BoxGeom);
		Solver->RegisterObject(Proxy2);
		AdvanceAndWait(Solver, DeltaTime);
		AdvanceAndWait(Solver, DeltaTime);

		TestUserData->SetData_GT(HandleExternal2, TestString2);
		REQUIRE(*TestUserData->GetData_GT(HandleExternal2) == TestString2);

		// And the old index still shows nothing (unless it was reused by
		// the new particle, in which case it correctly shows new data).
		const FMockHandle NewHandle(HandleExternal2.UniqueIdx());
		if (OldUniqueIdx == HandleExternal2.UniqueIdx())
		{
			// Index was reused — must see the new data, not the old
			REQUIRE(*TestUserData->GetData_GT(OldHandle) == TestString2);
		}
		else
		{
			// Index was not reused — old slot must still be empty
			REQUIRE(TestUserData->GetData_GT(OldHandle) == nullptr);
		}

		Solver->UnregisterObject(Proxy2);
		AdvanceAndWait(Solver, DeltaTime);
		AdvanceAndWait(Solver, DeltaTime);
	}

	Solver->WaitOnPendingTasks_External();
	Module->DestroySolver(Solver);
}
