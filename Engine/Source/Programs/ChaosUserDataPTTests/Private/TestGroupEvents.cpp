// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestCommon/Initialization.h"

#include <catch2/catch_test_macros.hpp>

GROUP_BEFORE_GLOBAL(Catch::DefaultGroup)
{
	InitAll(/*bAllowLogging=*/true, /*bMultithreaded=*/false);
}

GROUP_AFTER_GLOBAL(Catch::DefaultGroup)
{
	CleanupAll();
}
