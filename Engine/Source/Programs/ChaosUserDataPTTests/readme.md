ChaosUserDataPTTests
--------------------

This is a collection of tests for the ChaosUserDataPT plugin

# Running the Tests

## From VS

- Set `ChaosUserDataPTTests` as the startup project and set
Solution Configuration to a `Development` (or any config that
starts with `Development`, ie `Development Editor` etc).

- Compile and debug.

## From UShell

```ps1
.build target ChaosUserDataPTTests
.run target ChaosUserDataPTTests
```